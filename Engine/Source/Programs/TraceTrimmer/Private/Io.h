// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#if PLATFORM_DESKTOP

#if PLATFORM_WINDOWS

////////////////////////////////////////////////////////////////////////////////////////////////////

#include "Windows/AllowWindowsPlatformTypes.h"
#include <Windows.h>
#include "Windows/HideWindowsPlatformTypes.h"

typedef PTRINT FileHandle;
constexpr FileHandle InvalidFileHandle = FileHandle(-1);

////////////////////////////////////////////////////////////////////////////////////////////////////

inline bool IsValidFileHandle(FileHandle Handle)
{
	return Handle >= 0;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

inline FileHandle GetStdOut()
{
	return FileHandle(GetStdHandle(STD_OUTPUT_HANDLE));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

inline FileHandle GetStdIn()
{
	return FileHandle(GetStdHandle(STD_INPUT_HANDLE));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

inline FileHandle GetStdErr()
{
	return FileHandle(GetStdHandle(STD_ERROR_HANDLE));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

inline bool IsTty(FileHandle Handle)
{
	return GetFileType(HANDLE(Handle)) == FILE_TYPE_CHAR;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

inline int32 FileRead(FileHandle Handle, void* Data, int Size)
{
	DWORD BytesIn = 0;
	if (!ReadFile(HANDLE(Handle), Data, Size, &BytesIn, nullptr))
	{
		return -1;
	}
	return int32(BytesIn);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

inline int FileWrite(FileHandle Handle, const void* Data, int32 Size)
{
	DWORD BytesOut;
	if (!WriteFile(HANDLE(Handle), Data, Size, &BytesOut, nullptr))
	{
		return -1;
	}
	return int32(BytesOut);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

inline FileHandle OpenFile(const TCHAR* Path, bool Writeable)
{
	HANDLE Out = CreateFile(Path,
			Writeable ? GENERIC_WRITE : GENERIC_READ,
			FILE_SHARE_READ,
			nullptr,
			Writeable ? CREATE_ALWAYS : OPEN_EXISTING,
			0, nullptr);
	return (Out == INVALID_HANDLE_VALUE) ? FileHandle(-1) : FileHandle(Out);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

inline void CloseFile(FileHandle Handle)
{
	CloseHandle(HANDLE(Handle));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

#endif // PLATFORM_WINDOWS

#if PLATFORM_UNIX || PLATFORM_MAC

////////////////////////////////////////////////////////////////////////////////////////////////////

#include <fcntl.h>
#include <unistd.h>

typedef int FileHandle;
constexpr FileHandle InvalidFileHandle = FileHandle(-1);

////////////////////////////////////////////////////////////////////////////////////////////////////

inline bool IsValidFileHandle(FileHandle Handle)
{
	return Handle >= 0;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

inline FileHandle GetStdOut()
{
	return STDOUT_FILENO;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

inline FileHandle GetStdIn()
{
	return STDIN_FILENO;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

inline FileHandle GetStdErr()
{
	return STDERR_FILENO;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

inline bool IsTty(FileHandle Handle)
{
	return isatty(Handle);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

inline int32 FileRead(FileHandle Handle, void* Data, int Size)
{
	return read(Handle, Data, Size);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

inline int FileWrite(FileHandle Handle, const void* Data, int32 Size)
{
	return write(Handle, Data, Size);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

inline FileHandle OpenFile(const TCHAR* Path, bool Writeable)
{
	char InnerPath[512];
	for (ANSICHAR& Out : InnerPath)
	{
		Out = ANSICHAR(*Path++ & 0x7f);
		if (Out == '\0')
		{
			break;
		}
	}

	int Flags = Writeable ? O_CREAT|O_WRONLY|O_TRUNC : O_RDONLY;
	int Mode = S_IRUSR|S_IWUSR|S_IRGRP|S_IWGRP|S_IROTH|S_IWOTH;
	return open(InnerPath, Flags, Mode);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

inline void CloseFile(FileHandle Handle)
{
	close(Handle);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

#endif // PLATFORM_UNIX || PLATFORM_MAC

#endif // PLATFORM_DESKTOP
