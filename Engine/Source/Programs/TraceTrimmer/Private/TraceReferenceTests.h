// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

namespace UE::Trace::Tests
{

void TraceTestReferences(const TCHAR* TraceFile);

} // namespace UE::Trace::Tests
