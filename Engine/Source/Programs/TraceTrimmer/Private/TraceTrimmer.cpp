// Copyright Epic Games, Inc. All Rights Reserved.

#include "TraceTrimmer.h"
#include "TraceReferenceTests.h"

#include "Containers/AnsiString.h"
#include "HAL/PlatformFile.h"
#include "HAL/PlatformFileManager.h"
#include "Misc/CommandLine.h"
#include "RequiredProgramMainCPPInclude.h"
#include "Templates/UniquePtr.h"

#include "Io.h"
#include "TraceWriterTests.h"

// TraceAnalysis
#include "Trace/DataStream.h"
#include "Trace/OutDataStream.h"

// TraceServices
#include "TraceServices/TraceTrimmer.h"

#include <stdio.h>
#include <limits>

DEFINE_LOG_CATEGORY(LogTraceTrimmer);

IMPLEMENT_APPLICATION(TraceTrimmer, "TraceTrimmer");

#ifndef TRACE_TRIMMER_HAS_TRACE_WRITER_TESTS
#define TRACE_TRIMMER_HAS_TRACE_WRITER_TESTS 1
#endif // TRACE_TRIMMER_HAS_TRACE_WRITER_TESTS

namespace UE::TraceTrimmer
{

////////////////////////////////////////////////////////////////////////////////////////////////////

static void PrintUsage()
{
	puts("Usage:");
	//puts("  TraceTrimmer [-i=][<in_trace_file>] [-tracehost=... -traceid=...] [-o=...] [<options...>]");
	puts("  TraceTrimmer [-i=][<in_trace_file>] [-o=...] [<options...>]");
	puts("Where:");
	puts("  -i=<in_trace_file> : the input *.utrace file");
	//puts("  -tracehost=<host:port> : the host address of the input trace server");
	//puts("  -traceid=<id> : the id of the input trace (identified by the trace server)");
	puts("  -o=<out_trace_file> : the output *.utrace file; defaults to <in_trace_file>.out.utrace");
	//puts("  -outTraceHost=<out_host> : the host address of the output trace server");
	puts("  -startTime=<time> : the start time filter, in seconds");
	puts("  -endTime=<time> : the end time filter, in seconds");
	puts("  -include=<events> : a comma separated list of trace events to include");
	puts("  -exclude=<events> : a comma separated list of trace events to exclude");
	puts("Note:");
	puts("    The include/exclude events are specified as \"logger.event\" wildcard patterns, ex.: \"$trace.*,cpu.*\".");
#if TRACE_TRIMMER_HAS_TRACE_WRITER_TESTS
	puts("To run the TraceWriter tests:");
	puts("  TraceTrimmer -tests[=<out_path>]");
	puts("Where:");
	puts("  <out_path> : the path where to write the generated trace files");
#endif // TRACE_TRIMMER_HAS_TRACE_WRITER_TESTS
}

////////////////////////////////////////////////////////////////////////////////////////////////////

struct FCmdLineParams
{
	FString InputTraceFile;
	FString InputTraceServer;
	uint32 InputTraceId = 0;

	FString OutputTraceFile;
	FString OutputTraceServer;

	TraceServices::FTrimParameters TrimParams;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

#if TRACE_TRIMMER_HAS_TRACE_WRITER_TESTS
static int32 TryRunTests(int32 ArgC, TCHAR const* const* ArgV)
{
	if (ArgC == 2 &&
		(FCString::Stricmp(TEXT("-tests"), ArgV[1]) == 0 ||
		 FCString::Strnicmp(TEXT("-tests="), ArgV[1], 7) == 0))
	{
		FString TestsOutputPath;
		if (FCString::Strlen(ArgV[1]) > 7)
		{
			TestsOutputPath = ArgV[1] + 7;
		}

		IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
		FPaths::NormalizeDirectoryName(TestsOutputPath);
		PlatformFile.CreateDirectoryTree(*TestsOutputPath);

		UE::Trace::Tests::TraceWriterTest0(*(TestsOutputPath / TEXT("t0.utrace")));
		UE::Trace::Tests::TraceWriterTest1(*(TestsOutputPath / TEXT("t1.utrace")));
		UE::Trace::Tests::TraceWriterTest2a(*(TestsOutputPath / TEXT("t2a.utrace")));
		UE::Trace::Tests::TraceWriterTest2b(*(TestsOutputPath / TEXT("t2b.utrace")));
		UE::Trace::Tests::TraceWriterTest3(*(TestsOutputPath / TEXT("t3.utrace")));
		UE::Trace::Tests::TraceWriterTestCpuProfiler(*(TestsOutputPath / TEXT("t4.utrace")));
		UE::Trace::Tests::TraceTestReferences(*TestsOutputPath);

		return 1;
	}

	return 0;
}
#endif // TRACE_TRIMMER_HAS_TRACE_WRITER_TESTS

////////////////////////////////////////////////////////////////////////////////////////////////////

static int32 ParseCommandLine(FCmdLineParams& Parameters, int32 ArgC, TCHAR const* const* ArgV)
{
	if (ArgC < 2)
	{
		PrintUsage();
		return 0;
	}

	int32 StartArgIndex = 1;
	if (ArgV[1][0] != TEXT('-'))
	{
		Parameters.InputTraceFile = ArgV[1];
		StartArgIndex = 2;
	}
	for (int32 ArgIndex = StartArgIndex; ArgIndex < ArgC; ++ArgIndex)
	{
		const TCHAR* Arg = ArgV[ArgIndex];

		if (FCString::Strnicmp(TEXT("-i="), Arg, 3) == 0)
		{
			if (!Parameters.InputTraceFile.IsEmpty())
			{
				LogTraceTrimmerError("The input trace file cannot be specified multiple times!");
				return -1;
			}
			if (!Parameters.InputTraceServer.IsEmpty())
			{
				LogTraceTrimmerError("Cannot specify both an input trace file and an input trace server!");
				return -1;
			}
			Parameters.InputTraceFile = Arg + 3;
		}
#if 0 // not yet supported
		else if (FCString::Strnicmp(TEXT("-tracehost="), Arg, 11) == 0)
		{
			if (!Parameters.InputTraceServer.IsEmpty())
			{
				LogTraceTrimmerError("The input trace server cannot be specified multiple times!");
				return -1;
			}
			if (!Parameters.InputTraceFile.IsEmpty())
			{
				LogTraceTrimmerError("Cannot specify both an input trace file and an input trace server / trace id!");
				return -1;
			}
			Parameters.InputTraceServer = Arg + 11;
		}
		else if (FCString::Strnicmp(TEXT("-traceid="), Arg, 9) == 0)
		{
			if (Parameters.InputTraceId != 0)
			{
				LogTraceTrimmerError("The input trace id cannot be specified multiple times!");
				return -1;
			}
			if (!Parameters.InputTraceFile.IsEmpty())
			{
				LogTraceTrimmerError("Cannot specify both an input trace file and an input trace server / trace id!");
				return -1;
			}
			if (FCString::Strnicmp(TEXT("0x"), Arg + 9, 2) == 0)
			{
				Parameters.InputTraceId = FParse::HexNumber(Arg + 11);
			}
			else
			{
				Parameters.InputTraceId = FCString::Atoi(Arg + 9);
			}
		}
#endif
		else if (FCString::Strnicmp(TEXT("-o="), Arg, 3) == 0)
		{
			if (!Parameters.OutputTraceFile.IsEmpty())
			{
				LogTraceTrimmerError("The output trace file cannot be specified multiple times!");
				return -1;
			}
			if (!Parameters.OutputTraceServer.IsEmpty())
			{
				LogTraceTrimmerError("Cannot specify both an output trace file and an output trace server!");
				return -1;
			}
			Parameters.OutputTraceFile = Arg + 3;
		}
#if 0 // not yet supported
		else if (FCString::Strnicmp(TEXT("-outTraceHost="), Arg, 14) == 0)
		{
			if (!Parameters.OutputTraceServer.IsEmpty())
			{
				LogTraceTrimmerError("The output trace server cannot be specified multiple times!");
				return -1;
			}
			if (!Parameters.OutputTraceFile.IsEmpty())
			{
				LogTraceTrimmerError("Cannot specify both an output trace file and an output trace server!");
				return -1;
			}
			Parameters.OutputTraceServer = Arg + 14;
		}
#endif
		else if (FCString::Strnicmp(TEXT("-startTime="), Arg, 11) == 0)
		{
			Parameters.TrimParams.StartTime = FCString::Atof(Arg + 11);
		}
		else if (FCString::Strnicmp(TEXT("-endTime="), Arg, 9) == 0)
		{
			Parameters.TrimParams.EndTime = FCString::Atof(Arg + 9);
		}
		else if (FCString::Strnicmp(TEXT("-include="), Arg, 9) == 0)
		{
			Parameters.TrimParams.Include = FAnsiString(Arg + 9);
		}
		else if (FCString::Strnicmp(TEXT("-exclude="), Arg, 9) == 0)
		{
			Parameters.TrimParams.Exclude = FAnsiString(Arg + 9);
		}
		else
		{
			LogTraceTrimmerWarning("Unknown cmd line argument '%s'!", Arg);
		}
	}

	if (Parameters.InputTraceFile.IsEmpty() && Parameters.InputTraceServer.IsEmpty())
	{
		LogTraceTrimmerError("The input trace is not specified!");
		return -1;
	}

	if (Parameters.OutputTraceFile.IsEmpty() && Parameters.OutputTraceServer.IsEmpty())
	{
		if (!Parameters.InputTraceFile.IsEmpty())
		{
			Parameters.OutputTraceFile = FPaths::GetBaseFilename(Parameters.InputTraceFile, false) + TEXT(".out.utrace");
			FPaths::NormalizeFilename(Parameters.OutputTraceFile);
			LogTraceTrimmerWarning("The output trace is not specified. Will default to: \"%s\"", *Parameters.OutputTraceFile);
		}
		else
		{
			LogTraceTrimmerError("The input trace is not a file and the output trace file is not specified!");
			return -1;
		}
	}

	return 1;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TUniquePtr<UE::Trace::IInDataStream> CreateInputStream(FCmdLineParams& Parameters)
{
	FileHandle Input = -1;
	if (!Parameters.InputTraceFile.IsEmpty())
	{
		Input = OpenFile(*Parameters.InputTraceFile, false);
		if (Input < 0)
		{
			LogTraceTrimmerError("Cannot open the input trace file ('%s')!", *Parameters.InputTraceFile);
			return nullptr;
		}
	}

	class FInputDataStream : public UE::Trace::IInDataStream
	{
	public:
		FInputDataStream(FileHandle InHandle)
			: Handle(InHandle)
		{
		}

		virtual ~FInputDataStream()
		{
			Close();
		}

		virtual int32 Read(void* Data, uint32 Size) override
		{
			return FileRead(Handle, Data, Size);
		}

		virtual void Close() override
		{
			if (Handle >= 0)
			{
				CloseFile(Handle);
				Handle = -1;
			}
		}

		virtual bool WaitUntilReady() override
		{
			return true;
		}

	private:
		FileHandle Handle = -1;
	};

	TUniquePtr<UE::Trace::IInDataStream> InputDataStream = MakeUnique<FInputDataStream>(Input);
	return InputDataStream;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TUniquePtr<UE::Trace::IOutDataStream> CreateOutputStream(FCmdLineParams& Parameters)
{
	FileHandle Output = -1;
	if (!Parameters.OutputTraceFile.IsEmpty())
	{
		Output = OpenFile(*Parameters.OutputTraceFile, true);
		if (Output < 0)
		{
			LogTraceTrimmerError("Cannot write the output trace file ('%s')!", *Parameters.OutputTraceFile);
			return nullptr;
		}
	}

	class FOutputDataStream : public UE::Trace::IOutDataStream
	{
	public:
		FOutputDataStream(FileHandle InHandle)
			: Handle(InHandle)
		{
		}

		virtual ~FOutputDataStream()
		{
			Close();
		}

		virtual int32 Write(void* Data, uint32 Size) override
		{
			return FileWrite(Handle, Data, Size);
		}

		virtual void Close() override
		{
			if (Handle >= 0)
			{
				CloseFile(Handle);
				Handle = -1;
			}
		}

		virtual bool WaitUntilReady() override
		{
			return true;
		}

	private:
		FileHandle Handle = -1;
	};

	TUniquePtr<UE::Trace::IOutDataStream> OutputDataStream = MakeUnique<FOutputDataStream>(Output);
	return OutputDataStream;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

int32 Main(int32 ArgC, TCHAR const* const* ArgV)
{
	int32 Result;

#if TRACE_TRIMMER_HAS_TRACE_WRITER_TESTS
	Result = TryRunTests(ArgC, ArgV);
	if (Result != 0)
	{
		return Result;
	}
#endif // TRACE_TRIMMER_HAS_TRACE_WRITER_TESTS

	UE::TraceTrimmer::FCmdLineParams Parameters;

	Result = ParseCommandLine(Parameters, ArgC, ArgV);
	if (Result <= 0)
	{
		return Result;
	}

	TUniquePtr<UE::Trace::IInDataStream> InputDataStream = CreateInputStream(Parameters);
	if (!InputDataStream)
	{
		return -1;
	}

	TUniquePtr<UE::Trace::IOutDataStream> OutputDataStream = CreateOutputStream(Parameters);
	if (!OutputDataStream)
	{
		return -1;
	}

	return TraceServices::Trim(Parameters.TrimParams, *InputDataStream, *OutputDataStream);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::TraceTrimmer

////////////////////////////////////////////////////////////////////////////////////////////////////

INT32_MAIN_INT32_ARGC_TCHAR_ARGV()
{
	int32 Result = UE::TraceTrimmer::Main(ArgC, ArgV);

	// This ensures FUObjectArray::RemoveUObjectDeleteListener() is not called
	// from FSparseDelegateStorage::FObjectListener::~FObjectListener().
	RequestEngineExit(TEXT("TraceTrimmer completed"));

	return Result;
}

////////////////////////////////////////////////////////////////////////////////////////////////////
////////////////////////////////////////////////////////////////////////////////////////////////////
