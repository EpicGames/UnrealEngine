// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Logging/LogCategory.h"

DECLARE_LOG_CATEGORY_EXTERN(LogTraceTrimmer, Log, All);

#define LogTraceTrimmerError(Format, ...) \
	{ \
		wprintf(L"Error: " Format "\n", ##__VA_ARGS__); \
		UE_LOG(LogTraceTrimmer, Error, TEXT(Format), ##__VA_ARGS__); \
	}
#define LogTraceTrimmerWarning(Format, ...) \
	{ \
		wprintf(L"Warning: " Format "\n", ##__VA_ARGS__); \
		UE_LOG(LogTraceTrimmer, Warning, TEXT(Format), ##__VA_ARGS__); \
	}
#define LogTraceTrimmerMessage(Format, ...) \
	{ \
		wprintf(L"" Format "\n", ##__VA_ARGS__); \
		UE_LOG(LogTraceTrimmer, Log, TEXT(Format), ##__VA_ARGS__); \
	}
