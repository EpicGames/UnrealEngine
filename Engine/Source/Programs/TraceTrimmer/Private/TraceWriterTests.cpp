// Copyright Epic Games, Inc. All Rights Reserved.

#include "TraceWriterTests.h"

//#include "BuildSettings.h"
#include "HAL/PlatformTime.h"
#include "Misc/App.h"
#include "Misc/Guid.h"
#include "ProfilingDebugging/MiscTrace.h"

// TraceAnalysis
#include "Trace/TraceWriter.h"
#include "Trace/OutDataStream.h"

// TraceTrimmer
#include "TraceTrimmer.h"

UE_DISABLE_OPTIMIZATION

namespace UE::Trace::Tests
{

////////////////////////////////////////////////////////////////////////////////////////////////////

static uint64 GTestStartTime = 0;

static void BeginTest(const TCHAR* TraceFile)
{
	LogTraceTrimmerMessage("Writing %s...", TraceFile);

	GTestStartTime = FPlatformTime::Cycles64();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

static void EndTest(FTraceWriter& Writer)
{
	const uint64 TestDuration = FPlatformTime::Cycles64() - GTestStartTime;
	const double TestDurationSeconds = double(TestDuration) * FPlatformTime::GetSecondsPerCycle64();

	while (true)
	{
		FString Err = Writer.GetLastError();
		if (Err.IsEmpty())
		{
			break;
		}
		LogTraceTrimmerError("%s", *Err);
	}

	LogTraceTrimmerMessage("  done (%.6f seconds)", TestDurationSeconds);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void TraceWriterTest0(const TCHAR* TraceFile)
{
	BeginTest(TraceFile);

	FFileOutDataStream DataStream;
	if (!DataStream.Open(TraceFile))
	{
		return;
	}

	FTraceWriter Writer(DataStream);

	Writer.Begin();
	Writer.End();

	EndTest(Writer);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

static void WriteDiagnosticsSession2Event(FTraceWriter& Writer)
{
	constexpr int InstanceIdSize = 4;
	FGuid InstanceGuid = FApp::GetInstanceId();
	uint32 InstanceId[InstanceIdSize];
	for (int Index = 0; Index < InstanceIdSize; ++Index)
	{
		InstanceId[Index] = InstanceGuid[Index];
	}

	FAnsiStringView Platform = UE_STRINGIZE(UBT_COMPILED_PLATFORM);
	FAnsiStringView AppName = UE_APP_NAME;
	FStringView ProjectName = TEXTVIEW("");
	const TCHAR* EngineVersion = TEXT("5.8"); // BuildSettings::GetEngineVersionString();

	// Tests that the writer declares a valid Diagnostics.Session2 event.
	const uint32 DiagnosticsSession2EventId = Writer.DeclareDiagnosticsSession2Event();
	const FTraceWriterEventInfo* EventInfo = Writer.GetEventInfo(DiagnosticsSession2EventId);
	check(EventInfo != nullptr);
	check(EnumHasAllFlags(EventInfo->GetFlags(), ETraceWriterEventFlags::ImportantNoSync));

	Writer.SetCurrentThreadImportants();
	Writer.WriteEvent(DiagnosticsSession2EventId)
		.Field(ANSITEXTVIEW("ConfigurationType"), uint8(EBuildConfiguration::Test)) // uint8(FApp::GetBuildConfiguration())
		.Field(ANSITEXTVIEW("TargetType"), uint8(EBuildTargetType::Program)) // uint8(FApp::GetBuildTargetType())
		.Field(ANSITEXTVIEW("Changelist"), 0) // BuildSettings::GetCurrentChangelist()
		.Field(ANSITEXTVIEW("InstanceId"), TConstArrayView<uint32>(InstanceId, InstanceIdSize))
		.Field(ANSITEXTVIEW("Platform"), Platform)
		.Field(ANSITEXTVIEW("AppName"), AppName)
		//.Field(ANSITEXTVIEW("ProjectName"), ProjectName)
		//.Field(ANSITEXTVIEW("Branch"), TEXTVIEW(""))
		//.Field(ANSITEXTVIEW("BuildVersion"), TEXTVIEW(""))
		.Field(ANSITEXTVIEW("EngineVersion"), EngineVersion)
		.Field(ANSITEXTVIEW("CommandLine"), TEXTVIEW("TraceTrimmer.exe -tests"))
		//.Field(ANSITEXTVIEW("VFSPaths"), ANSITEXTVIEW("")) // BuildSettings::GetVfsPaths();
		.End();
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// FMiscWriter
////////////////////////////////////////////////////////////////////////////////////////////////////

class FMiscWriter
{
public:
	FMiscWriter(FTraceWriter& InWriter)
		: Writer(InWriter)
	{
	}
	virtual ~FMiscWriter()
	{
	}

	void DeclareEvents()
	{
		DeclareFrameEvents();
	}
	void DeclareEvents2()
	{
		DeclareFrameEvents2();
	}

	void DeclareFrameEvents();
	void DeclareFrameEvents2();

	void WriteBeginFrameEvent(uint64 Cycle, uint8 FrameType);
	void WriteBeginFrameEvent2(uint64 Cycle, uint8 FrameType);
	void WriteEndFrameEvent(uint64 Cycle, uint8 FrameType);
	void WriteEndFrameEvent2(uint64 Cycle, uint8 FrameType);

	void Test();
	void Test2();

private:
	FTraceWriter& Writer;
	uint32 EventId_Misc_BeginFrame = 0;
	uint32 EventId_Misc_EndFrame = 0;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMiscWriter::DeclareFrameEvents()
{
	EventId_Misc_BeginFrame =
	Writer.DeclareEvent(ANSITEXTVIEW("Misc"), ANSITEXTVIEW("BeginFrame"))
		.Field(ANSITEXTVIEW("Cycle"), ETraceWriterFieldType::Uint64)
		.Field(ANSITEXTVIEW("FrameType"), ETraceWriterFieldType::Uint8)
		.End();

	EventId_Misc_EndFrame =
	Writer.DeclareEvent(ANSITEXTVIEW("Misc"), ANSITEXTVIEW("EndFrame"))
		.Field(ANSITEXTVIEW("Cycle"), ETraceWriterFieldType::Uint64)
		.Field(ANSITEXTVIEW("FrameType"), ETraceWriterFieldType::Uint8)
		.End();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMiscWriter::DeclareFrameEvents2()
{
	EventId_Misc_BeginFrame =
	UE_ADHOC_TRACE_BEGIN_DECLARE_EVENT(Writer, "Misc", "BeginFrame", None) // Sync
		UE_ADHOC_TRACE_DECLARE_FIELD(uint64, "Cycle")
		UE_ADHOC_TRACE_DECLARE_FIELD(uint8, "FrameType")
	UE_ADHOC_TRACE_END_DECLARE_EVENT

	EventId_Misc_EndFrame =
	UE_ADHOC_TRACE_BEGIN_DECLARE_EVENT(Writer, "Misc", "EndFrame", None) // Sync
		UE_ADHOC_TRACE_DECLARE_FIELD(uint64, "Cycle")
		UE_ADHOC_TRACE_DECLARE_FIELD(uint8, "FrameType")
	UE_ADHOC_TRACE_END_DECLARE_EVENT

#if 0 // TBD !!!
	EventId_Misc_BeginFrame =
	UE_ADHOC_TRACE_BEGIN_DECLARE_EVENT(Writer, Misc, BeginFrame, Sync)
		UE_ADHOC_TRACE_DECLARE_FIELD(uint64, Cycle)
		UE_ADHOC_TRACE_DECLARE_FIELD(uint8, FrameType)
	UE_ADHOC_TRACE_END_DECLARE_EVENT
		
	EventId_Misc_EndFrame =
	UE_ADHOC_TRACE_BEGIN_DECLARE_EVENT(Writer, Misc, EndFrame, Sync)
		UE_ADHOC_TRACE_DECLARE_FIELD(uint64, Cycle)
		UE_ADHOC_TRACE_DECLARE_FIELD(uint8, FrameType)
	UE_ADHOC_TRACE_END_DECLARE_EVENT()
#endif
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMiscWriter::WriteBeginFrameEvent(uint64 Cycle, uint8 FrameType)
{
	Writer.WriteEvent(EventId_Misc_BeginFrame)
		.Field(0, Cycle)
		.Field(1, FrameType)
		.End();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMiscWriter::WriteBeginFrameEvent2(uint64 Cycle, uint8 FrameType)
{
	UE_ADHOC_TRACE_LOG(Writer, EventId_Misc_BeginFrame, Cycle, FrameType);

	//UE_ADHOC_TRACE_WRITE_EVENT(Writer, EventId_Misc_BeginFrame, Cycle, FrameType); // TBD !!!

	// TBD !!!
	//UE_ADHOC_TRACE_BEGIN_WRITE_EVENT(Writer, EventId_Misc_BeginFrame)
	//	UE_ADHOC_TRACE_WRITE_FIELD(0, Cycle)
	//	UE_ADHOC_TRACE_WRITE_FIELD(1, FrameType)
	//UE_ADHOC_TRACE_END_WRITE_EVENT()

	// TBD !!!
	//UE_ADHOC_TRACE_BEGIN_WRITE_EVENT(Writer, EventId_Misc_BeginFrame)
	//	UE_ADHOC_TRACE_WRITE_FIELD_BY_NAME("Cycle", Cycle)
	//	UE_ADHOC_TRACE_WRITE_FIELD_BY_NAME("FrameType", FrameType)
	//UE_ADHOC_TRACE_END_WRITE_EVENT()
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMiscWriter::WriteEndFrameEvent(uint64 Cycle, uint8 FrameType)
{
	Writer.WriteEvent(EventId_Misc_EndFrame)
		.Field(0, Cycle)
		.Field(1, FrameType)
		.End();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMiscWriter::WriteEndFrameEvent2(uint64 Cycle, uint8 FrameType)
{
	UE_ADHOC_TRACE_LOG(Writer, EventId_Misc_EndFrame, Cycle, FrameType);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMiscWriter::Test()
{
	const uint32 GameThreadId = Writer.RegisterThread(ANSITEXTVIEW("GameThread"));
	const uint32 RenderThreadId = Writer.RegisterThread(ANSITEXTVIEW("RenderThread"));

	DeclareEvents();

	const uint64 BaseTimestamp = Writer.GetStartTime();
	const uint64 OneSecond = Writer.GetTimeFrequency();

	constexpr uint64 NumPackets = 10;
	constexpr uint64 FramesPerPacket = 100;

	const uint64 MaxFrameDuration = (16 * OneSecond) / 1000;
	FMath::RandInit(42);

	for (uint64 PacketIndex = 0; PacketIndex < NumPackets; ++PacketIndex)
	{
		Writer.SetCurrentThread(GameThreadId);
		for (uint64 FrameIndex = 0; FrameIndex < FramesPerPacket; ++FrameIndex)
		{
			const uint64 Timestamp = BaseTimestamp + MaxFrameDuration * (PacketIndex * FramesPerPacket + FrameIndex);
			const uint64 Duration = 6 * MaxFrameDuration / 7 + MaxFrameDuration * (uint64)FMath::Rand() / ((uint64)RAND_MAX * 7);

			WriteBeginFrameEvent(Timestamp, (uint8)TraceFrameType_Game);
			WriteEndFrameEvent(Timestamp + Duration, (uint8)TraceFrameType_Game);
		}

		Writer.SetCurrentThread(RenderThreadId);
		for (uint64 FrameIndex = 0; FrameIndex < FramesPerPacket; ++FrameIndex)
		{
			const uint64 Timestamp = BaseTimestamp + MaxFrameDuration / 10 + MaxFrameDuration * (PacketIndex * FramesPerPacket + FrameIndex);
			const uint64 Duration = 7 * MaxFrameDuration / 8 + MaxFrameDuration * (uint64)FMath::Rand() / ((uint64)RAND_MAX * 8);

			WriteBeginFrameEvent(Timestamp, (uint8)TraceFrameType_Rendering);
			WriteEndFrameEvent(Timestamp + Duration, (uint8)TraceFrameType_Rendering);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FMiscWriter::Test2()
{
	const uint32 GameThreadId = Writer.RegisterThread("GameThread");
	const uint32 RenderThreadId = Writer.RegisterThread("RenderThread");

	DeclareEvents2();

	const uint64 BaseTimestamp = Writer.GetStartTime();
	const uint64 OneSecond = Writer.GetTimeFrequency();

	constexpr uint64 NumPackets = 10;
	constexpr uint64 FramesPerPacket = 100;

	const uint64 MaxFrameDuration = (16 * OneSecond) / 1000;
	FMath::RandInit(42);

	for (uint64 PacketIndex = 0; PacketIndex < NumPackets; ++PacketIndex)
	{
		Writer.SetCurrentThread(GameThreadId);
		for (uint64 FrameIndex = 0; FrameIndex < FramesPerPacket; ++FrameIndex)
		{
			const uint64 Timestamp = BaseTimestamp + MaxFrameDuration * (PacketIndex * FramesPerPacket + FrameIndex);
			const uint64 Duration = 6 * MaxFrameDuration / 7 + MaxFrameDuration * (uint64)FMath::Rand() / ((uint64)RAND_MAX * 7);

			WriteBeginFrameEvent(Timestamp, (uint8)TraceFrameType_Game);
			WriteEndFrameEvent(Timestamp + Duration, (uint8)TraceFrameType_Game);
		}

		Writer.SetCurrentThread(RenderThreadId);
		for (uint64 FrameIndex = 0; FrameIndex < FramesPerPacket; ++FrameIndex)
		{
			const uint64 Timestamp = BaseTimestamp + MaxFrameDuration / 10 + MaxFrameDuration * (PacketIndex * FramesPerPacket + FrameIndex);
			const uint64 Duration = 7 * MaxFrameDuration / 8 + MaxFrameDuration * (uint64)FMath::Rand() / ((uint64)RAND_MAX * 8);

			WriteBeginFrameEvent2(Timestamp, (uint8)TraceFrameType_Rendering);
			WriteEndFrameEvent2(Timestamp + Duration, (uint8)TraceFrameType_Rendering);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
////////////////////////////////////////////////////////////////////////////////////////////////////

void TraceWriterTest1(const TCHAR* TraceFile)
{
	BeginTest(TraceFile);

	FFileOutDataStream DataStream;
	if (!DataStream.Open(TraceFile))
	{
		return;
	}

	FTraceWriter Writer(DataStream);
	Writer.Begin();
	WriteDiagnosticsSession2Event(Writer);
	Writer.End();

	EndTest(Writer);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void TraceWriterTest2a(const TCHAR* TraceFile)
{
	BeginTest(TraceFile);

	FFileOutDataStream DataStream;
	if (!DataStream.Open(TraceFile))
	{
		return;
	}

	FTraceWriter Writer(DataStream);
	Writer.Begin();
	WriteDiagnosticsSession2Event(Writer);
	FMiscWriter MiscWriter(Writer);
	MiscWriter.Test();
	Writer.End();

	EndTest(Writer);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void TraceWriterTest2b(const TCHAR* TraceFile)
{
	BeginTest(TraceFile);

	FFileOutDataStream DataStream;
	if (!DataStream.Open(TraceFile))
	{
		return;
	}

	FTraceWriter Writer(DataStream);
	Writer.Begin();
	WriteDiagnosticsSession2Event(Writer);
	FMiscWriter MiscWriter(Writer);
	MiscWriter.Test2();
	Writer.End();

	EndTest(Writer);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

static void WriteTestEvents(FTraceWriter& Writer, ETraceWriterEventFlags EventFlags)
{
	const uint32 GameThreadId = Writer.RegisterThread(ANSITEXTVIEW("GameThread"));

	// Test if threads are cached.
	const uint32 GameThreadId2 = Writer.RegisterThread(ANSITEXTVIEW("GameThread"));
	check(GameThreadId2 == GameThreadId);

	FAnsiStringView Logger("Test");

	FAnsiString SimpleTypesEventName("SimpleTypes");
	FAnsiString ArrayTypesEventName("ArrayTypes");

	bool bIsImportant = false;
	if (EnumHasAnyFlags(EventFlags, ETraceWriterEventFlags::Important))
	{
		bIsImportant = true;
		SimpleTypesEventName += "_Important";
		ArrayTypesEventName += "_Important";
	}
	if (EnumHasAnyFlags(EventFlags, ETraceWriterEventFlags::NoSync))
	{
		SimpleTypesEventName += "_NoSync";
		ArrayTypesEventName += "_NoSync";
	}
	else
	{
		SimpleTypesEventName += "_Sync";
		ArrayTypesEventName += "_Sync";
	}

	const uint32 EventId_Test_SimpleTypes = Writer.DeclareEvent(Logger, SimpleTypesEventName, EventFlags)
		.Field(ANSITEXTVIEW("Bool"),    ETraceWriterFieldType::Bool) // Protocol7 doesn't make any difference between Bool and Uint8 :(
		.Field(ANSITEXTVIEW("Uint8"),   ETraceWriterFieldType::Uint8)
		.Field(ANSITEXTVIEW("Uint16"),  ETraceWriterFieldType::Uint16)
		.Field(ANSITEXTVIEW("Uint32"),  ETraceWriterFieldType::Uint32)
		.Field(ANSITEXTVIEW("Uint64"),  ETraceWriterFieldType::Uint64)
		.Field(ANSITEXTVIEW("Int8"),    ETraceWriterFieldType::Int8)
		.Field(ANSITEXTVIEW("Int16"),   ETraceWriterFieldType::Int16)
		.Field(ANSITEXTVIEW("Int32"),   ETraceWriterFieldType::Int32)
		.Field(ANSITEXTVIEW("Int64"),   ETraceWriterFieldType::Int64)
		.Field(ANSITEXTVIEW("Float32"), ETraceWriterFieldType::Float32)
		.Field(ANSITEXTVIEW("Float64"), ETraceWriterFieldType::Float64)
		.Field(ANSITEXTVIEW("Pointer"), ETraceWriterFieldType::Pointer) // Protocol7 doesn't make any difference between Pointer and Uint64 :(
		.End();

	const uint32 EventId_Test_ArrayTypes = Writer.DeclareEvent(Logger, ArrayTypesEventName, EventFlags)
		.Field(ANSITEXTVIEW("Uint8Array"),   ETraceWriterFieldType::ArrayFlag | ETraceWriterFieldType::Uint8)
		.Field(ANSITEXTVIEW("Uint16Array"),  ETraceWriterFieldType::ArrayFlag | ETraceWriterFieldType::Uint16)
		.Field(ANSITEXTVIEW("Uint32Array"),  ETraceWriterFieldType::ArrayFlag | ETraceWriterFieldType::Uint32)
		.Field(ANSITEXTVIEW("Uint64Array"),  ETraceWriterFieldType::ArrayFlag | ETraceWriterFieldType::Uint64)
		.Field(ANSITEXTVIEW("Int8Array"),    ETraceWriterFieldType::ArrayFlag | ETraceWriterFieldType::Int8)
		.Field(ANSITEXTVIEW("Int16Array"),   ETraceWriterFieldType::ArrayFlag | ETraceWriterFieldType::Int16)
		.Field(ANSITEXTVIEW("Int32Array"),   ETraceWriterFieldType::ArrayFlag | ETraceWriterFieldType::Int32)
		.Field(ANSITEXTVIEW("Int64Array"),   ETraceWriterFieldType::ArrayFlag | ETraceWriterFieldType::Int64)
		.Field(ANSITEXTVIEW("Float32Array"), ETraceWriterFieldType::ArrayFlag | ETraceWriterFieldType::Float32)
		.Field(ANSITEXTVIEW("Float64Array"), ETraceWriterFieldType::ArrayFlag | ETraceWriterFieldType::Float64)
		.Field(ANSITEXTVIEW("PointerArray"), ETraceWriterFieldType::ArrayFlag | ETraceWriterFieldType::Pointer)
		.End();

	if (bIsImportant)
	{
		Writer.SetCurrentThreadImportants();
	}
	else
	{
		Writer.SetCurrentThread(GameThreadId);
	}

	Writer.WriteEvent(EventId_Test_SimpleTypes)
		.Field("Bool", false)
		.Field("Uint8", (uint8)0x10)
		.Field("Uint16", (uint16)0x3210)
		.Field("Uint32", 0x76543210u)
		.Field("Uint64", 0xFEDCBA9876543210ull)
		.Field("Int8", (int8)0x10)
		.Field("Int16", (int16)0x3210)
		.Field("Int32", (int32)0x76543210)
		.Field("Int64", (int64)0xFEDCBA9876543210)
		.Field("Float32", 43.21f)
		.Field("Float64", 12.34)
		.Field("Pointer", &Writer)
		.End();

	Writer.WriteEvent(EventId_Test_SimpleTypes)
		.Field("Bool", true)
		.Field("Uint8", (uint8)0x01)
		.Field("Uint16", (uint16)0x1032)
		.Field("Uint32", 0x10765432u)
		.Field("Uint64", 0x10FEDCBA98765432ull)
		.Field("Int8", (int8)0x01)
		.Field("Int16", (int16)0x1032)
		.Field("Int32", (int32)0x10765432)
		.Field("Int64", (int64)0x10FEDCBA98765432)
		.Field("Float32", 34.12f)
		.Field("Float64", 21.43)
		.Field("Pointer", &Writer)
		.End();

	TArray<uint8>  Uint8Array  = { 0, 1, 10, 42, 0xFF };
	TArray<uint16> Uint16Array = { 0, 1, 0x00FF, 0xFF00, 0xFFFF };
	TArray<uint32> Uint32Array = { 0u, 1u, 0x76543210u, 0xFEDCBA98u, 0xFFFFFFFFu };
	TArray<uint64> Uint64Array = { 0ull, 1ull, 0xFEDCBA9876543210ull, 0x0123456789ABCDEFull, ~0ull };

	TArray<int8>  Int8Array  = { 0, 1, -1, 100, MAX_int8, -100, MIN_int8 };
	TArray<int16> Int16Array = { 0, 1, -1, 10000, MAX_int16, -10000, MIN_int16 };
	TArray<int32> Int32Array = { 0, 1, -1, 100000, 987654321, MAX_int32, -100000, -987654321, MIN_int32 };
	TArray<int64> Int64Array = { 0, 1, -1, 1000000, MAX_int64, -1000000, MIN_int64 };

	TArray<float>   FloatArray = { 0.0f, 0.1f, 1.0f, -0.1f, -1.0f, MAX_flt, MIN_flt };
	TArray<double> DoubleArray = { 0.0, 0.1, 1.0, -0.1, -1.0, MAX_dbl, MIN_dbl };

	TArray<void*> PointerArray = { &Writer, &Writer };

	// Arrays specified as (const void* Data, uint32 DataSize) pairs.
	Writer.WriteEvent(EventId_Test_ArrayTypes)
		.Field("Uint8Array",   Uint8Array.GetData(),   Uint8Array.Num()   * sizeof(uint8))
		.Field("Uint16Array",  Uint16Array.GetData(),  Uint16Array.Num()  * sizeof(uint16))
		.Field("Uint32Array",  Uint32Array.GetData(),  Uint32Array.Num()  * sizeof(uint32))
		.Field("Uint64Array",  Uint64Array.GetData(),  Uint64Array.Num()  * sizeof(uint64))
		.Field("Int8Array",    Int8Array.GetData(),    Int8Array.Num()    * sizeof(int8))
		.Field("Int16Array",   Int16Array.GetData(),   Int16Array.Num()   * sizeof(int16))
		.Field("Int32Array",   Int32Array.GetData(),   Int32Array.Num()   * sizeof(int32))
		.Field("Int64Array",   Int64Array.GetData(),   Int64Array.Num()   * sizeof(int64))
		.Field("Float32Array", FloatArray.GetData(),   FloatArray.Num()   * sizeof(float))
		.Field("Float64Array", DoubleArray.GetData(),  DoubleArray.Num()  * sizeof(double))
		.Field("PointerArray", PointerArray.GetData(), PointerArray.Num() * sizeof(void*))
		.End();

	// Arrays specified as TConstArrayView<>.
	Writer.WriteEvent(EventId_Test_ArrayTypes)
		.Field("Uint8Array",   TConstArrayView<uint8>  (Uint8Array.GetData(),   Uint8Array.Num()))
		.Field("Uint16Array",  TConstArrayView<uint16> (Uint16Array.GetData(),  Uint16Array.Num()))
		.Field("Uint32Array",  TConstArrayView<uint32> (Uint32Array.GetData(),  Uint32Array.Num()))
		.Field("Uint64Array",  TConstArrayView<uint64> (Uint64Array.GetData(),  Uint64Array.Num()))
		.Field("Int8Array",    TConstArrayView<int8>   (Int8Array.GetData(),    Int8Array.Num()))
		.Field("Int16Array",   TConstArrayView<int16>  (Int16Array.GetData(),   Int16Array.Num()))
		.Field("Int32Array",   TConstArrayView<int32>  (Int32Array.GetData(),   Int32Array.Num()))
		.Field("Int64Array",   TConstArrayView<int64>  (Int64Array.GetData(),   Int64Array.Num()))
		.Field("Float32Array", TConstArrayView<float>  (FloatArray.GetData(),   FloatArray.Num()))
		.Field("Float64Array", TConstArrayView<double> (DoubleArray.GetData(),  DoubleArray.Num()))
		.Field("PointerArray", TConstArrayView<void*>  (PointerArray.GetData(), PointerArray.Num()))
		.End();

	// Arrays specified as TArrayView<>.
	Writer.WriteEvent(EventId_Test_ArrayTypes)
		.Field("Uint8Array",   TArrayView<uint8>  (Uint8Array.GetData(),   Uint8Array.Num()))
		.Field("Uint16Array",  TArrayView<uint16> (Uint16Array.GetData(),  Uint16Array.Num()))
		.Field("Uint32Array",  TArrayView<uint32> (Uint32Array.GetData(),  Uint32Array.Num()))
		.Field("Uint64Array",  TArrayView<uint64> (Uint64Array.GetData(),  Uint64Array.Num()))
		.Field("Int8Array",    TArrayView<int8>   (Int8Array.GetData(),    Int8Array.Num()))
		.Field("Int16Array",   TArrayView<int16>  (Int16Array.GetData(),   Int16Array.Num()))
		.Field("Int32Array",   TArrayView<int32>  (Int32Array.GetData(),   Int32Array.Num()))
		.Field("Int64Array",   TArrayView<int64>  (Int64Array.GetData(),   Int64Array.Num()))
		.Field("Float32Array", TArrayView<float>  (FloatArray.GetData(),   FloatArray.Num()))
		.Field("Float64Array", TArrayView<double> (DoubleArray.GetData(),  DoubleArray.Num()))
		.Field("PointerArray", TArrayView<void*>  (PointerArray.GetData(), PointerArray.Num()))
		.End();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

static void WriteAdhocMacrosTest(FTraceWriter& Writer)
{
	const uint32 TestEvent1 =
	UE_ADHOC_TRACE_BEGIN_DECLARE_EVENT(Writer, "Test", "TestEvent1_Important", ImportantNoSync)
		UE_ADHOC_TRACE_DECLARE_FIELD(bool, "BoolField")
		UE_ADHOC_TRACE_DECLARE_FIELD(uint8, "Uint8Field")
		UE_ADHOC_TRACE_DECLARE_FIELD(uint64, "Uint64Field")
		UE_ADHOC_TRACE_DECLARE_FIELD(void*, "PointerField")
		UE_ADHOC_TRACE_DECLARE_FIELD(uint8[], "ByteArrayField")
		UE_ADHOC_TRACE_DECLARE_FIELD(uint64[], "Uint64ArrayField")
		UE_ADHOC_TRACE_DECLARE_FIELD(void*[], "PointerArrayField")
	UE_ADHOC_TRACE_END_DECLARE_EVENT

	const uint32 TestEvent2 =
	UE_ADHOC_TRACE_BEGIN_DECLARE_EVENT(Writer, "Test", "TestEvent2_Sync", None) // Sync
		UE_ADHOC_TRACE_DECLARE_FIELD(bool, "BoolField")
		UE_ADHOC_TRACE_DECLARE_FIELD(uint8, "Uint8Field")
		UE_ADHOC_TRACE_DECLARE_FIELD(uint64, "Uint64Field")
		UE_ADHOC_TRACE_DECLARE_FIELD(void*, "PointerField")
		UE_ADHOC_TRACE_DECLARE_FIELD(uint8[], "ByteArrayField")
		UE_ADHOC_TRACE_DECLARE_FIELD(uint64[], "Uint64ArrayField")
		UE_ADHOC_TRACE_DECLARE_FIELD(void*[], "PointerArrayField")
	UE_ADHOC_TRACE_END_DECLARE_EVENT

	const uint32 TestEvent3 =
	UE_ADHOC_TRACE_BEGIN_DECLARE_EVENT(Writer, "Test", "TestEvent3_NoSync", NoSync)
		UE_ADHOC_TRACE_DECLARE_FIELD(bool, "BoolField")
		UE_ADHOC_TRACE_DECLARE_FIELD(uint8, "Uint8Field")
		UE_ADHOC_TRACE_DECLARE_FIELD(uint64, "Uint64Field")
		UE_ADHOC_TRACE_DECLARE_FIELD(const void*, "PointerField")
		UE_ADHOC_TRACE_DECLARE_FIELD(uint8[], "ByteArrayField")
		UE_ADHOC_TRACE_DECLARE_FIELD(uint64[], "Uint64ArrayField")
		UE_ADHOC_TRACE_DECLARE_FIELD(void*[], "PointerArrayField")
	UE_ADHOC_TRACE_END_DECLARE_EVENT

	TArray<uint8>   Uint8Array = { 0, 1, 10, 42, 0xFF };
	TArray<uint64> Uint64Array = { 0ull, 1ull, 0xFEDCBA9876543210ull, 0x0123456789ABCDEFull, ~0ull };

	TArray<void*> PointerArray = { &Writer, &Writer };

	Writer.SetCurrentThreadImportants();

	UE_ADHOC_TRACE_LOG(Writer, TestEvent1,
		(bool)true,
		(uint8)42,
		0x0123456789ABCDEFull,
		&Writer,
		TArrayView<uint8>(Uint8Array.GetData(), Uint8Array.Num()),
		TArrayView<uint64>(Uint64Array.GetData(), Uint64Array.Num()),
		TArrayView<void*>(PointerArray.GetData(), PointerArray.Num()))

	const uint32 GameThreadId = Writer.RegisterThread(ANSITEXTVIEW("GameThread"));

	Writer.SetCurrentThread(GameThreadId);

	UE_ADHOC_TRACE_LOG(Writer, TestEvent2,
		(bool)true,
		(uint8)42,
		0x0123456789ABCDEFull,
		&Writer,
		TArrayView<uint8>(Uint8Array.GetData(), Uint8Array.Num()),
		TArrayView<uint64>(Uint64Array.GetData(), Uint64Array.Num()),
		TArrayView<void*>(PointerArray.GetData(), PointerArray.Num()))

	UE_ADHOC_TRACE_LOG(Writer, TestEvent3,
		(bool)true,
		(uint8)42,
		0x0123456789ABCDEFull,
		&Writer,
		TConstArrayView<uint8>(Uint8Array.GetData(), Uint8Array.Num()),
		TConstArrayView<uint64>(Uint64Array.GetData(), Uint64Array.Num()),
		TConstArrayView<void*>(PointerArray.GetData(), PointerArray.Num()))
}

////////////////////////////////////////////////////////////////////////////////////////////////////

static void WriteScopedTestEvents(FTraceWriter& Writer)
{
	const uint32 TestScopedEvent1 =
	UE_ADHOC_TRACE_BEGIN_DECLARE_EVENT(Writer, "Test", "ScopedEvent1_NoSync", NoSync)
		UE_ADHOC_TRACE_DECLARE_FIELD(uint32, "Id")
		UE_ADHOC_TRACE_DECLARE_FIELD(AnsiString, "Name")
	UE_ADHOC_TRACE_END_DECLARE_EVENT

	const uint32 TestScopedEvent2 =
	UE_ADHOC_TRACE_BEGIN_DECLARE_EVENT(Writer, "Test", "ScopedEvent2_Sync", None) // Sync
		UE_ADHOC_TRACE_DECLARE_FIELD(uint32, "Id")
		UE_ADHOC_TRACE_DECLARE_FIELD(AnsiString, "Name")
	UE_ADHOC_TRACE_END_DECLARE_EVENT

	const uint32 GameThreadId = Writer.RegisterThread(ANSITEXTVIEW("GameThread"));
	Writer.SetCurrentThread(GameThreadId);

	Writer.WriteEnterScopeEvent();
	Writer.WriteEvent(TestScopedEvent1)
		.Field("Id", 1)
		.Field("Name", "abc")
		.End();
	//...
	Writer.WriteLeaveScopeEvent();

	Writer.WriteEnterScopeEvent();
	Writer.WriteEvent(TestScopedEvent2)
		.Field("Id", 2)
		.Field("Name", "xyz")
		.End();
	//...
	Writer.WriteLeaveScopeEvent();

	Writer.WriteStampedEnterScopeEvent(Writer.GetTime());
	Writer.WriteEvent(TestScopedEvent1)
		.Field("Id", 3)
		.Field("Name", "abc2")
		.End();
	//...
	Writer.WriteStampedLeaveScopeEvent(Writer.GetTime());

	Writer.WriteStampedEnterScopeEvent(Writer.GetTime());
	Writer.WriteEvent(TestScopedEvent2)
		.Field("Id", 4)
		.Field("Name", "xyz2")
		.End();
	//...
	Writer.WriteStampedLeaveScopeEvent(Writer.GetTime());
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void TraceWriterTest3(const TCHAR* TraceFile)
{
	BeginTest(TraceFile);

	FFileOutDataStream DataStream;
	if (!DataStream.Open(TraceFile))
	{
		return;
	}

	FTraceWriter Writer(DataStream);
	Writer.Begin();
	WriteDiagnosticsSession2Event(Writer);
	WriteTestEvents(Writer, ETraceWriterEventFlags::None); // Sync
	WriteTestEvents(Writer, ETraceWriterEventFlags::ImportantNoSync);
	WriteTestEvents(Writer, ETraceWriterEventFlags::NoSync);
	WriteAdhocMacrosTest(Writer);
	WriteScopedTestEvents(Writer);
	Writer.End();

	EndTest(Writer);
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// FCpuProfilerEventBatchWriter
////////////////////////////////////////////////////////////////////////////////////////////////////

class FCpuProfilerEventBatchWriter
{
public:
	FCpuProfilerEventBatchWriter()
	{
	}
	virtual ~FCpuProfilerEventBatchWriter()
	{
		if (Buffer)
		{
			FMemory::Free(Buffer);
		}
	}

	void Begin(uint32 MinInitialSize = 8 * 1024)
	{
		UsedSize = 0;
		if (MinInitialSize > AllocatedSize)
		{
			Resize(MinInitialSize);
		}
		LastCpuEventTimestamp = 0;
	}

	void BeginCpuEvent(uint32 SpecId, uint64 Timestamp);
	void EndCpuEvent(uint64 Timestamp);

	TConstArrayView<uint8> End()
	{
		return TConstArrayView<uint8>(Buffer, UsedSize);
	}

	void Resize(uint32 NewSize)
	{
		if (NewSize == AllocatedSize)
		{
			return;
		}
		uint8* NewBuffer = nullptr;
		if (NewSize > 0)
		{
			NewBuffer = (uint8*)FMemory::Malloc(NewSize);
			if (UsedSize > 0)
			{
				FMemory::Memcpy(NewBuffer, Buffer, FMath::Min(UsedSize, NewSize));
			}
		}
		if (Buffer)
		{
			FMemory::Free(Buffer);
		}
		Buffer = NewBuffer;
		AllocatedSize = NewSize;
		if (UsedSize > AllocatedSize)
		{
			UsedSize = AllocatedSize;
		}
	}

	uint8* GetBuffer(uint32 Required)
	{
		constexpr uint32 PageSize = 4 * 1024;
		if (UsedSize + Required > AllocatedSize)
		{
			Resize((UsedSize + Required + PageSize - 1) & ~(PageSize - 1));
		}
		return Buffer + UsedSize;
	}

private:
	uint8* Buffer = nullptr;
	uint32 AllocatedSize = 0;
	uint32 UsedSize = 0;
	uint64 LastCpuEventTimestamp = 0;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

void FCpuProfilerEventBatchWriter::BeginCpuEvent(uint32 InSpecId, uint64 InTimestamp)
{
	uint8* StartCursor = GetBuffer(10 + 5);
	uint8* Cursor = StartCursor;

	uint64 CycleDiff = InTimestamp - LastCpuEventTimestamp;
	LastCpuEventTimestamp = InTimestamp;
	
	// Write Timestamp
	FTraceUtils::Encode7bit((CycleDiff << 2) | 1ull, Cursor);
	
	// Write SpecId
	FTraceUtils::Encode7bit((InSpecId << 1) | 0, Cursor);

	UsedSize += (Cursor - StartCursor);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FCpuProfilerEventBatchWriter::EndCpuEvent(uint64 Timestamp)
{
	uint8* StartCursor = GetBuffer(10);
	uint8* Cursor = StartCursor;

	uint64 CycleDiff = Timestamp - LastCpuEventTimestamp;
	LastCpuEventTimestamp = Timestamp;

	// Write Timestamp
	FTraceUtils::Encode7bit(CycleDiff << 2, Cursor);

	UsedSize += (Cursor - StartCursor);
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// FCpuProfilerWriter
////////////////////////////////////////////////////////////////////////////////////////////////////

class FCpuProfilerWriter
{
public:
	FCpuProfilerWriter(FTraceWriter& InWriter)
		: Writer(InWriter)
	{
	}
	virtual ~FCpuProfilerWriter()
	{
	}

	void DeclareEvents();

	void WriteEventSpecEvent(uint32 SpecId, FAnsiStringView Name, FAnsiStringView File = FAnsiStringView(), uint32 Line = 0);
	void WriteEventBatchEvent(TConstArrayView<uint8> Data);

	void Test();

private:
	FTraceWriter& Writer;
	uint32 EventId_EventSpec = 0;
	uint32 EventId_MetadataSpec = 0;
	uint32 EventId_Metadata = 0;
	uint32 EventId_EventBatchV3 = 0;
	uint32 EventId_EndThread = 0;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

void FCpuProfilerWriter::DeclareEvents()
{
	if (EventId_EventSpec != 0)
	{
		return;
	}

	EventId_EventSpec =
	UE_ADHOC_TRACE_BEGIN_DECLARE_EVENT(Writer, "CpuProfiler", "EventSpec", ImportantNoSync)
		UE_ADHOC_TRACE_DECLARE_FIELD(uint32, "Id")
		UE_ADHOC_TRACE_DECLARE_FIELD(UE::Trace::AnsiString, "Name")
		UE_ADHOC_TRACE_DECLARE_FIELD(UE::Trace::AnsiString, "File")
		UE_ADHOC_TRACE_DECLARE_FIELD(uint32, "Line")
	UE_ADHOC_TRACE_END_DECLARE_EVENT

	EventId_MetadataSpec =
	UE_ADHOC_TRACE_BEGIN_DECLARE_EVENT(Writer, "CpuProfiler", "MetadataSpec", ImportantNoSync)
		UE_ADHOC_TRACE_DECLARE_FIELD(uint32, "Id")
		UE_ADHOC_TRACE_DECLARE_FIELD(UE::Trace::AnsiString, "Name")
		UE_ADHOC_TRACE_DECLARE_FIELD(UE::Trace::WideString, "NameFormat")
		UE_ADHOC_TRACE_DECLARE_FIELD(uint8[], "FieldNames")
	UE_ADHOC_TRACE_END_DECLARE_EVENT

	EventId_Metadata =
	UE_ADHOC_TRACE_BEGIN_DECLARE_EVENT(Writer, "CpuProfiler", "Metadata", NoSync)
		UE_ADHOC_TRACE_DECLARE_FIELD(uint32, "Id")
		UE_ADHOC_TRACE_DECLARE_FIELD(uint32, "SpecId")
		UE_ADHOC_TRACE_DECLARE_FIELD(uint8[], "Metadata")
	UE_ADHOC_TRACE_END_DECLARE_EVENT

	EventId_EventBatchV3 =
	UE_ADHOC_TRACE_BEGIN_DECLARE_EVENT(Writer, "CpuProfiler", "EventBatchV3", NoSync)
		UE_ADHOC_TRACE_DECLARE_FIELD(uint8[], "Data")
	UE_ADHOC_TRACE_END_DECLARE_EVENT

	EventId_EndThread =
	UE_ADHOC_TRACE_BEGIN_DECLARE_EVENT(Writer, "CpuProfiler", "EndThread", NoSync)
		UE_ADHOC_TRACE_DECLARE_FIELD(uint64, "Cycle") // added in UE 5.4
	UE_ADHOC_TRACE_END_DECLARE_EVENT
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FCpuProfilerWriter::WriteEventSpecEvent(uint32 SpecId, FAnsiStringView Name, FAnsiStringView File, uint32 Line)
{
	Writer.WriteEvent(EventId_EventSpec)
		.Field(0, SpecId)
		.Field(1, Name)
		.Field(2, File)
		.Field(3, Line)
		.End();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FCpuProfilerWriter::WriteEventBatchEvent(TConstArrayView<uint8> Data)
{
	Writer.WriteEvent(EventId_EventBatchV3)
		.Field(0, Data)
		.End();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FCpuProfilerWriter::Test()
{
	DeclareEvents();
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// FTimingEventsWriter
////////////////////////////////////////////////////////////////////////////////////////////////////

class FTimingEventsWriter
{
public:
	FTimingEventsWriter(FTraceWriter& InWriter, FCpuProfilerWriter& InCpuProfilerWriter, FCpuProfilerEventBatchWriter& InBatchWriter, uint32 InNumSpecs)
		: Writer(InWriter)
		, CpuProfilerWriter(InCpuProfilerWriter)
		, BatchWriter(InBatchWriter)
		, NumSpecs(InNumSpecs)
	{
		const uint64 OneSecond = Writer.GetTimeFrequency();
		MinParentEventDuration = OneSecond / 1000000; // 1us
	}
	~FTimingEventsWriter()
	{
	}

	void BeginTimingEvent(uint64 StartTime);
	void EndTimingEvent(uint64 EndTime);
	void GenerateTimingEvents(uint32 Depth, uint64 IntervalStartTime, uint64 IntervalDuration);

private:
	FTraceWriter& Writer;
	FCpuProfilerWriter& CpuProfilerWriter;
	FCpuProfilerEventBatchWriter& BatchWriter;
	uint32 TotalTimingEvents = 0;
	static constexpr uint32 FlushAfterNumTimingEvents = 200;
	uint64 MinParentEventDuration = 0;
	uint32 NumSpecs = 0;
	static constexpr uint32 MaxDepth = 16;
	static constexpr uint32 MaxChildTimingEvents = 10;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimingEventsWriter::BeginTimingEvent(uint64 StartTime)
{
	++TotalTimingEvents;
	if (TotalTimingEvents % FlushAfterNumTimingEvents == 0)
	{
		CpuProfilerWriter.WriteEventBatchEvent(BatchWriter.End());
		BatchWriter.Begin();
	}

	uint32 SpecId = 1 + FMath::RandHelper(NumSpecs);
	check(SpecId >= 1 && SpecId <= NumSpecs);
	BatchWriter.BeginCpuEvent(SpecId, StartTime);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimingEventsWriter::EndTimingEvent(uint64 EndTime)
{
	BatchWriter.EndCpuEvent(EndTime);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTimingEventsWriter::GenerateTimingEvents(uint32 Depth, uint64 IntervalStartTime, uint64 IntervalDuration)
{
	if (IntervalDuration == 0)
	{
		return;
	}

	const uint64 IntervalEndTime = IntervalStartTime + IntervalDuration;

	uint64 TimingEventTime = IntervalStartTime + uint64(FMath::RandHelper64((int64)IntervalDuration /10));

	while (true)
	{
		const uint64 TimingEventDuration = uint64(FMath::RandHelper64((int64)IntervalDuration));
		if (TimingEventTime + TimingEventDuration > IntervalEndTime)
		{
			break;
		}

		BeginTimingEvent(TimingEventTime);

		if (Depth < MaxDepth && TimingEventDuration > MinParentEventDuration)
		{
			GenerateTimingEvents(Depth + 1, TimingEventTime, TimingEventDuration);
		}

		TimingEventTime += TimingEventDuration;
		EndTimingEvent(TimingEventTime);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
////////////////////////////////////////////////////////////////////////////////////////////////////

static void WriteCpuProfilerTest(FTraceWriter& Writer)
{
	FCpuProfilerWriter CpuProfilerWriter(Writer);
	FMiscWriter MiscWriter(Writer);

	const uint32 GameThreadId = Writer.RegisterThread(ANSITEXTVIEW("GameThread"));
	const uint32 RenderThreadId = Writer.RegisterThread(ANSITEXTVIEW("RenderThread"));

	CpuProfilerWriter.DeclareEvents();
	MiscWriter.DeclareFrameEvents();

	Writer.SetCurrentThreadImportants();

	constexpr uint32 NumSpecs = 26;
	for (uint32 SpecId = 1; SpecId <= NumSpecs; ++SpecId)
	{
		ANSICHAR Name[10];
		Name[0] = 'A' + SpecId - 1;
		Name[1] = 0;
		CpuProfilerWriter.WriteEventSpecEvent(SpecId, Name);
	}

	const uint64 BaseTimestamp = Writer.GetStartTime();
	const uint64 OneSecond = Writer.GetTimeFrequency();
	const uint64 OneMilliSecond = OneSecond / 1000;
	const uint64 OneMicroSecond = OneSecond / 1000000;

	uint64 MinFrameDuration = (10 * OneSecond) / 1000; // 10ms
	uint64 MaxFrameDuration = (20 * OneSecond) / 1000; // 20ms
	uint64 MaxDurationBetweenFrames = OneSecond / 1000; // 1ms

	FCpuProfilerEventBatchWriter BatchWriter;
	FTimingEventsWriter TimingEventsWriter(Writer, CpuProfilerWriter, BatchWriter, NumSpecs);

	auto GenerateFrames =
		[&]
		(uint64 StartTime, uint32 FrameCount, uint8 FrameType) -> uint64
		{
			uint64 CurrentTime = StartTime;
			uint32 NumTimingEvents = 0;

			for (uint32 FrameIndex = 0; FrameIndex < FrameCount; ++FrameIndex)
			{
				MiscWriter.WriteBeginFrameEvent(CurrentTime, FrameType);

				const uint64 FrameDuration = MinFrameDuration + FMath::RandHelper64(int64(MaxFrameDuration - MinFrameDuration));

				BatchWriter.Begin();
				TimingEventsWriter.GenerateTimingEvents(0, CurrentTime, FrameDuration);
				CpuProfilerWriter.WriteEventBatchEvent(BatchWriter.End());

				CurrentTime += FrameDuration;
				MiscWriter.WriteEndFrameEvent(CurrentTime, FrameType);

				CurrentTime += uint64(FMath::RandHelper64(MaxDurationBetweenFrames));
			}

			return CurrentTime;
		};

	FMath::RandInit(42);

	constexpr uint64 NumPackets = 10;
	constexpr uint64 FramesPerPacket = 100;

	uint64 CurrentTime = BaseTimestamp;

	for (uint64 PacketIndex = 0; PacketIndex < NumPackets; ++PacketIndex)
	{
		Writer.SetCurrentThread(GameThreadId);
		uint64 CurrentTimeA = GenerateFrames(CurrentTime, FramesPerPacket, (uint8)TraceFrameType_Game);

		Writer.SetCurrentThread(RenderThreadId);
		uint64 CurrentTimeB = GenerateFrames(CurrentTime, FramesPerPacket, (uint8)TraceFrameType_Rendering);

		CurrentTime = FMath::Max(CurrentTimeA, CurrentTimeB);
		CurrentTime += uint64(FMath::RandHelper64(MaxDurationBetweenFrames));
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void TraceWriterTestCpuProfiler(const TCHAR* TraceFile)
{
	BeginTest(TraceFile);

	FFileOutDataStream DataStream;
	if (!DataStream.Open(TraceFile))
	{
		return;
	}

	FTraceWriter Writer(DataStream);
	Writer.Begin();
	WriteDiagnosticsSession2Event(Writer);
	WriteCpuProfilerTest(Writer);
	Writer.End();

	EndTest(Writer);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Trace::Tests

UE_ENABLE_OPTIMIZATION
