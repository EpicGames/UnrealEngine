// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

namespace UE::Trace::Tests
{

void TraceWriterTest0(const TCHAR* TraceFile);
void TraceWriterTest1(const TCHAR* TraceFile);
void TraceWriterTest2a(const TCHAR* TraceFile);
void TraceWriterTest2b(const TCHAR* TraceFile);
void TraceWriterTest3(const TCHAR* TraceFile);
void TraceWriterTestCpuProfiler(const TCHAR* TraceFile);

} // namespace UE::Trace::Tests
