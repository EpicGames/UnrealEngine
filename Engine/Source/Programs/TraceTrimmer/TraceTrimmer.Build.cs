// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class TraceTrimmer : ModuleRules
{
	public TraceTrimmer(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicIncludePathModuleNames.Add("Launch");
		PrivateDependencyModuleNames.Add("Core");
		PrivateDependencyModuleNames.Add("Projects");
		PrivateDependencyModuleNames.Add("TraceAnalysis");
		PrivateDependencyModuleNames.Add("TraceServices");
	}
}
